// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class EnvironmentLightingViewer : ModuleRules
{
	public EnvironmentLightingViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"WorkspaceMenuStructure"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
                "Settings",
            }
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
                "AssetTools",
				"EditorWidgets",
			}
		);
	}
}
