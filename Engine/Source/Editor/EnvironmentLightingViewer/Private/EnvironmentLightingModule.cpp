// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentLightingModule.h"

#include "Delegates/Delegate.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "ISettingsModule.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "SEnvironmentLightingViewer.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "EnvironmentLightingViewer"

IMPLEMENT_MODULE(FEnvironmentLightingViewerModule, EnvironmentLightingViewer);


namespace EnvironmentLightingViewerModule
{
	static const FName EnvironmentLightingViewerApp = FName("EnvironmentLightingViewerApp");
}



TSharedRef<SDockTab> CreateEnvLightTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SEnvironmentLightingViewer)
		];
}



void FEnvironmentLightingViewerModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(EnvironmentLightingViewerModule::EnvironmentLightingViewerApp, FOnSpawnTab::CreateStatic(&CreateEnvLightTab))//TODO picker tab
		.SetDisplayName(NSLOCTEXT("EnvironmentLightingViewerApp", "TabTitle", "EnvironmentLighting Viewer"))
		.SetTooltipText(NSLOCTEXT("EnvironmentLightingViewerApp", "TooltipText", "Environment lighting window."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassViewer.TabIcon"));

	// TODO setting module ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
}

void FEnvironmentLightingViewerModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(EnvironmentLightingViewerModule::EnvironmentLightingViewerApp);
	}

	// Unregister the setting
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Editor", "EnvironmentLightingViewer");
	}
}

TSharedRef<SWidget> FEnvironmentLightingViewerModule::CreateEnvironmentLightingViewer()
{
	return SNew(SEnvironmentLightingViewer);
}

#undef LOCTEXT_NAMESPACE
