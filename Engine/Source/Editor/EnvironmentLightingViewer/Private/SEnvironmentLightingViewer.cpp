// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEnvironmentLightingViewer.h"

#include "Components/SkyLightComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/SkyAtmosphereComponent.h"
#include "Components/VolumetricCloudComponent.h"
#include "Components/ExponentialHeightFogComponent.h"
#include "Engine/ExponentialHeightFog.h"
#include "Engine/DirectionalLight.h"
#include "Engine/SkyLight.h"

#include "Modules/ModuleManager.h"
#include "Editor.h"
#include "SlateOptMacros.h"
#include "PropertyEditorModule.h"

#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SEnvironmentLightingViewer"

DEFINE_LOG_CATEGORY_STATIC(LogEditorEnvironmentLightingViewer, Log, All);

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SEnvironmentLightingViewer::Construct(const FArguments& InArgs)
{
	UWorld* World = GEditor->GetEditorWorldContext().World();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ComponentsAndActorsUseNameArea;

	for (int i = 0; i < ENVLIGHT_MAX_DETAILSVIEWS; ++i)
	{
		DetailsViews[i] = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsViews[i]->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SEnvironmentLightingViewer::GetIsPropertyVisible));
		DetailsViews[i]->SetDisableCustomDetailLayouts(true);	// This is to not have transforms and other special viewer
	}
	DefaultForegroundColor = DetailsViews[0]->GetColorAndOpacity();

	CheckBoxAtmosphericLightsOnly =	SNew(SCheckBox)
									.Padding(5.0f)
									.ToolTipText(LOCTEXT("CheckBoxAtmosphericLightsOnlyToolTip", "Wether to only present atmospheric lights."));

	ComboBoxDetailFilterOptions.Add(MakeShared<FString>(LOCTEXT("FilterMinimal",	"Minimal").ToString()));
	ComboBoxDetailFilterOptions.Add(MakeShared<FString>(LOCTEXT("FilterNormal",		"Normal").ToString()));
	ComboBoxDetailFilterOptions.Add(MakeShared<FString>(LOCTEXT("FilterNormalAdv",	"Normal+Advanced").ToString()));

	SelectedComboBoxDetailFilterOptions = 0;
	ComboBoxDetailFilter =	SNew(SComboBox<TSharedPtr<FString>>)
							.ToolTipText(LOCTEXT("ComboBoxDetailFilterTooTip", "Select the amount of details desired."))
							.OptionsSource(&ComboBoxDetailFilterOptions)
							.InitiallySelectedItem(ComboBoxDetailFilterOptions[SelectedComboBoxDetailFilterOptions])
							.OnSelectionChanged(this, &SEnvironmentLightingViewer::ComboBoxDetailFilterWidgetSelectionChanged)
							.OnGenerateWidget(this, &SEnvironmentLightingViewer::ComboBoxDetailFilterWidget)
							[
								SNew(STextBlock)
								.Text(this, &SEnvironmentLightingViewer::GetSelectedComboBoxDetailFilterTextLabel)
							];

	uint32 Zero = 0;
	ButtonCreateSkyLight = SNew(SButton)
					.HAlign(HAlign_Center)
					.OnClicked(this, &SEnvironmentLightingViewer::OnButtonCreateSkyLight)
					.Text(LOCTEXT("CreateSkyLight", "Create Sky Light"));
	ButtonCreateAtmosphericLight0 = SNew(SButton)
					.HAlign(HAlign_Center)
					.OnClicked(this, &SEnvironmentLightingViewer::OnButtonCreateDirectionalLight, Zero)
					.Text(LOCTEXT("CreateAtmosphericLight0", "Create Directional Light"));
	ButtonCreateSkyAtmosphere = SNew(SButton)
					.HAlign(HAlign_Center)
					.OnClicked(this, &SEnvironmentLightingViewer::OnButtonCreateSkyAtmosphere)
					.Text(LOCTEXT("CreateSkyAtmosphere", "Create Sky Atmosphere"));
	ButtonCreateVolumetricCloud= SNew(SButton)
					.HAlign(HAlign_Center)
					.OnClicked(this, &SEnvironmentLightingViewer::OnButtonCreateVolumetricCloud)
					.Text(LOCTEXT("CreateVolumetricCloud", "Create Volumetric Cloud"));
	ButtonCreateHeightFog= SNew(SButton)
					.HAlign(HAlign_Center)
					.OnClicked(this, &SEnvironmentLightingViewer::OnButtonCreateHeightFog)
					.Text(LOCTEXT("CreateHeightFog", "Create Height Fog"));

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 5.0f, 0.0f, 0.0f)
		[
			SNew(SWrapBox)
			.UseAllottedSize(true)
			+SWrapBox::Slot()
			.Padding(5.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 5.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AtmosphericLightsOnly", "Only show atmospheric lights  "))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 5.0f, 0.0f, 0.0f)
				[
					CheckBoxAtmosphericLightsOnly->AsShared()
				]
			]
			+SWrapBox::Slot()
			.Padding(5.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				ComboBoxDetailFilter->AsShared()
			]
			+SWrapBox::Slot()
			.Padding(5.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				ButtonCreateSkyLight->AsShared()
			]
			+SWrapBox::Slot()
			.Padding(5.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				ButtonCreateAtmosphericLight0->AsShared()
			]
			+SWrapBox::Slot()
			.Padding(5.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				ButtonCreateSkyAtmosphere->AsShared()
			]
			+SWrapBox::Slot()
			.Padding(5.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				ButtonCreateVolumetricCloud->AsShared()
			]
			+SWrapBox::Slot()
			.Padding(5.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				ButtonCreateHeightFog->AsShared()
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 5.0f, 0.0f, 0.0f)
		[
			SNew(SSeparator)
			.Thickness(5.f)
			.Orientation(EOrientation::Orient_Horizontal)
		]
		+ SVerticalBox::Slot()
		//.AutoHeight()			// Cannot use that otherwise scrollbars disapear.
		.Padding(0.0f, 5.0f, 0.0f, 0.0f)
		[
			SNew(SScrollBox)
			.Orientation(Orient_Vertical)
			.ScrollBarAlwaysVisible(false)
			+ SScrollBox::Slot()
			[
				SNew(SWrapBox)
				.UseAllottedSize(true)
				.PreferredSize(384.0f)
				+SWrapBox::Slot()
				.FillLineWhenSizeLessThan(384)
				.Padding(5.0f, 20.0f, 5.0f, 20.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					DetailsViews[0]->AsShared()
				]
				+SWrapBox::Slot()
				.Padding(5.0f, 20.0f, 5.0f, 20.0f)
				.FillLineWhenSizeLessThan(384)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					DetailsViews[1]->AsShared()
				]
				+SWrapBox::Slot()
				.Padding(5.0f, 20.0f, 5.0f, 20.0f)
				.FillLineWhenSizeLessThan(384)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					DetailsViews[2]->AsShared()
				]
				+SWrapBox::Slot()
				.Padding(5.0f, 20.0f, 5.0f, 20.0f)
				.FillLineWhenSizeLessThan(384)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					DetailsViews[3]->AsShared()
				]
				+SWrapBox::Slot()
				.Padding(5.0f, 20.0f, 5.0f, 20.0f)
				.FillLineWhenSizeLessThan(384)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					DetailsViews[4]->AsShared()
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> SEnvironmentLightingViewer::GetContent()
{
	return SharedThis(this);
}

SEnvironmentLightingViewer::~SEnvironmentLightingViewer()
{
	for (int i = 0; i < ENVLIGHT_MAX_DETAILSVIEWS; ++i)
	{
		if (DetailsViews[i].IsValid())
		{
			DetailsViews[i].Reset();
		}
	}
}

void SEnvironmentLightingViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return;
	}

	int NumDetailsView = 0;

	auto GetAtmosphericLight = [&](const uint8 DesiredLightIndex)
	{
		UDirectionalLightComponent* SelectedAtmosphericLight = nullptr;
		float SelectedLightLuminance = 0.0f;
		for (TObjectIterator<UDirectionalLightComponent> ComponentIt; ComponentIt; ++ComponentIt)
		{
			if (ComponentIt->GetWorld() == World && ComponentIt->IsRenderStateCreated())
			{
				UDirectionalLightComponent* AtmosphericLight = *ComponentIt;

				if ((CheckBoxAtmosphericLightsOnly->IsChecked() && !AtmosphericLight->IsUsedAsAtmosphereSunLight()) 
					|| AtmosphericLight->GetAtmosphereSunLightIndex() != DesiredLightIndex || !AtmosphericLight->GetVisibleFlag()
					/*|| AtmosphericLight->*/)
					continue;

				float LightLuminance = AtmosphericLight->GetColoredLightBrightness().GetLuminance();
				if (!SelectedAtmosphericLight ||					// Set it if null
					SelectedLightLuminance < LightLuminance)		// Or choose the brightest atmospheric light
				{
					SelectedAtmosphericLight = AtmosphericLight;
				}
			}
		}
		return SelectedAtmosphericLight;
	};

	FLinearColor DirLightColor = DefaultForegroundColor + FLinearColor(1.0f, 1.0f, 0.0f, 0.0f);
	FLinearColor SkyLightColor = DefaultForegroundColor + FLinearColor(0.0f, 1.0f, 1.0f, 0.0f);
	FLinearColor SkyAtmosColor = DefaultForegroundColor + FLinearColor(0.0f, 0.0f, 1.0f, 0.0f);
	FLinearColor VolCloudColor = DefaultForegroundColor;

	auto AddComponentDetailView = [&](auto* InComponent, const char* ComponentName, FLinearColor& ColorAndOpacity)
	{
		if (InComponent)
		{
			DetailsViews[NumDetailsView]->SetObject(InComponent);
			NumDetailsView++;
		}
	};

	UDirectionalLightComponent* AtmosphericLight0 = GetAtmosphericLight(0);
	AddComponentDetailView(AtmosphericLight0, "Directional Light 0", DirLightColor);
	ButtonCreateAtmosphericLight0->SetVisibility(AtmosphericLight0 ? EVisibility::Collapsed : EVisibility::Visible);

	USkyLightComponent* SkyLightComp = nullptr;
	for (TObjectIterator<USkyLightComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		if (ComponentIt->GetWorld() == World && ComponentIt->IsRenderStateCreated())
		{
			SkyLightComp = *ComponentIt;
			break;
		}
	}
	AddComponentDetailView(SkyLightComp, "Sky Light", SkyLightColor);
	ButtonCreateSkyLight->SetVisibility(SkyLightComp ? EVisibility::Collapsed : EVisibility::Visible);

	USkyAtmosphereComponent* SkyAtmosphereComp = nullptr;
	for (TObjectIterator<USkyAtmosphereComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		if (ComponentIt->GetWorld() == World && ComponentIt->IsRenderStateCreated())
		{
			SkyAtmosphereComp = *ComponentIt;
			break;
		}
	}
	AddComponentDetailView(SkyAtmosphereComp, "Sky Atmosphere", SkyAtmosColor);
	ButtonCreateSkyAtmosphere->SetVisibility(SkyAtmosphereComp ? EVisibility::Collapsed : EVisibility::Visible);

	UVolumetricCloudComponent* VolumetricCloudComp = nullptr;
	for (TObjectIterator<UVolumetricCloudComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		if (ComponentIt->GetWorld() == World && ComponentIt->IsRenderStateCreated())
		{
			VolumetricCloudComp = *ComponentIt;
			break;
		}
	}
	AddComponentDetailView(VolumetricCloudComp, "Volumetric Cloud", VolCloudColor);
	ButtonCreateVolumetricCloud->SetVisibility(VolumetricCloudComp ? EVisibility::Collapsed : EVisibility::Visible);

	UExponentialHeightFogComponent* HeightFogComp = nullptr;
	for (TObjectIterator<UExponentialHeightFogComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		if (ComponentIt->GetWorld() == World && ComponentIt->IsRenderStateCreated())
		{
			HeightFogComp = *ComponentIt;
			break;
		}
	}
	AddComponentDetailView(HeightFogComp, "Height Fog", VolCloudColor);
	ButtonCreateHeightFog->SetVisibility(HeightFogComp ? EVisibility::Collapsed : EVisibility::Visible);

	for (int i = NumDetailsView; i < ENVLIGHT_MAX_DETAILSVIEWS; ++i)
	{
		// If the details view selection is already empty, don't call SetObject again.  Calling SetObject
		// otherwise closes any active color picker (UE-121571).
		if (DetailsViews[i]->GetSelectedObjects().Num() > 0)
		{
			DetailsViews[i]->SetObject(nullptr);
		}
	}
}

FReply SEnvironmentLightingViewer::OnButtonCreateSkyLight()
{
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return FReply::Handled();
	}

	const FTransform Transform(FVector(0.0f, 0.0f, 0.0f));
	ASkyLight* SkyLight = Cast<ASkyLight>(GEditor->AddActor(World->GetCurrentLevel(), ASkyLight::StaticClass(), Transform));
	SkyLight->GetLightComponent()->SetMobility(EComponentMobility::Movable);
	SkyLight->GetLightComponent()->SetRealTimeCaptureEnabled(true);

	return FReply::Handled();
}

FReply SEnvironmentLightingViewer::OnButtonCreateDirectionalLight(uint32 Index)
{
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return FReply::Handled();
	}

	const FTransform Transform(FVector(0.0f, 0.0f, 0.0f));
	ADirectionalLight* DirectionalLight = Cast<ADirectionalLight>(GEditor->AddActor(World->GetCurrentLevel(), ADirectionalLight::StaticClass(), Transform));
	DirectionalLight->SetMobility(EComponentMobility::Movable);
	DirectionalLight->SetActorRotation(FRotator(329, 346, -105));
#if WITH_EDITORONLY_DATA
	DirectionalLight->GetComponent()->bAtmosphereSunLight = 1;
	DirectionalLight->GetComponent()->AtmosphereSunLightIndex = Index;
	// The render proxy is create right after AddActor, so we need to mark the render state as dirty again to get the new values set on the render side too.
	DirectionalLight->MarkComponentsRenderStateDirty();
#endif

	return FReply::Handled();
}

FReply SEnvironmentLightingViewer::OnButtonCreateSkyAtmosphere()
{
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return FReply::Handled();
	}

	const FTransform Transform(FVector(0.0f, 0.0f, 0.0f));
	ASkyAtmosphere* SkyLight = Cast<ASkyAtmosphere>(GEditor->AddActor(World->GetCurrentLevel(), ASkyAtmosphere::StaticClass(), Transform));

	return FReply::Handled();
}

FReply SEnvironmentLightingViewer::OnButtonCreateVolumetricCloud()
{
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return FReply::Handled();
	}

	const FTransform Transform(FVector(0.0f, 0.0f, 0.0f));
	AVolumetricCloud* SkyLight = Cast<AVolumetricCloud>(GEditor->AddActor(World->GetCurrentLevel(), AVolumetricCloud::StaticClass(), Transform));

	return FReply::Handled();
}

FReply SEnvironmentLightingViewer::OnButtonCreateHeightFog()
{
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return FReply::Handled();
	}

	const FTransform Transform(FVector(0.0f, 0.0f, 0.0f));
	AExponentialHeightFog* HeightFog = Cast<AExponentialHeightFog>(GEditor->AddActor(World->GetCurrentLevel(), AExponentialHeightFog::StaticClass(), Transform));

	return FReply::Handled();
}



TSharedRef<SWidget> SEnvironmentLightingViewer::ComboBoxDetailFilterWidget(TSharedPtr<FString> InItem)
{
	FString ItemString;
	if (InItem.IsValid())
	{
		ItemString = *InItem;
	}
	return SNew(STextBlock).Text(FText::FromString(*ItemString));
}
void SEnvironmentLightingViewer::ComboBoxDetailFilterWidgetSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	for (int32 i = 0; i < ComboBoxDetailFilterOptions.Num(); ++i)
	{
		if (ComboBoxDetailFilterOptions[i] == NewSelection)
		{
			SelectedComboBoxDetailFilterOptions = i;
			break;
		}
	}
	for (int i = 0; i < ENVLIGHT_MAX_DETAILSVIEWS; ++i)
	{
		DetailsViews[i]->Invalidate(EInvalidateWidgetReason::Paint);
		DetailsViews[i]->ForceRefresh();
	}
}
FText SEnvironmentLightingViewer::GetSelectedComboBoxDetailFilterTextLabel() const
{
	int32 ComboBoxDetailFilterOptionsNum = ComboBoxDetailFilterOptions.Num();
	int32 SelectionIndex = SelectedComboBoxDetailFilterOptions < 0 ? 0 : (SelectedComboBoxDetailFilterOptions < ComboBoxDetailFilterOptionsNum ? SelectedComboBoxDetailFilterOptions : ComboBoxDetailFilterOptionsNum-1);
	return FText::FromString(*ComboBoxDetailFilterOptions[SelectionIndex]);
}



bool SEnvironmentLightingViewer::GetIsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	const UClass* OwnerClass = PropertyAndParent.Property.GetOwner<UClass>();
	const UStruct* OwnerStruct = PropertyAndParent.Property.GetOwner<UStruct>();

	bool bShowAdvanced = SelectedComboBoxDetailFilterOptions == 0 || SelectedComboBoxDetailFilterOptions == 2;
	bool bShowMinimalOnly = SelectedComboBoxDetailFilterOptions == 0;

	if ((PropertyAndParent.Property.PropertyFlags & CPF_AdvancedDisplay) == CPF_AdvancedDisplay && !bShowAdvanced)
	{
		return false; // The user has decided to not show advanced or all properties
	}

	if (OwnerClass == USceneComponent::StaticClass())
	{
		return false; // No need for USceneComponent properties
	}

	if (OwnerClass == USkyLightComponent::StaticClass())
	{
		if (bShowMinimalOnly)
		{
			return PropertyAndParent.Property.GetNameCPP().Equals(TEXT("Intensity"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("LightColor"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("bLowerHemisphereIsBlack"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("LowerHemisphereColor"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("IndirectLightingIntensity"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("VolumetricScatteringIntensity"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("OcclusionMaxDistance"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("OcclusionExponent"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("MinOcclusion"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("OcclusionTint"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("CloudAmbientOcclusionStrength"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("CloudAmbientOcclusionApertureScale"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("bCloudAmbientOcclusion"));
		}
		return true;
	}
	else if (OwnerStruct && OwnerStruct->GetName().Equals(TEXT("TentDistribution")))
	{
		// This is only used to control the atmosphere absorption only layer in the atmosphere. So we trivially show according to the filter option.
		if (bShowMinimalOnly)
		{
			return false;
		}
		return true;
	}
	else if (OwnerClass == USkyAtmosphereComponent::StaticClass())
	{
		if (bShowMinimalOnly)
		{
			return PropertyAndParent.Property.GetNameCPP().Equals(TEXT("GroundAlbedo"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("MultiScatteringFactor"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("RayleighScattering"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("RayleighScatteringScale"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("MieScattering"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("MieAbsorptionScale"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("MieAnisotropy"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("OtherAbsorptionScale"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("OtherAbsorption"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("SkyLuminanceFactor"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("SkyAndAerialPerspectiveLuminanceFactor"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("HeightFogContribution"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("AerialPespectiveViewDistanceScale"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("AerialPerspectiveStartDepth"));
		}
		return true;
	}
	else if (OwnerClass == UVolumetricCloudComponent::StaticClass())
	{
		if (bShowMinimalOnly)
		{
			return PropertyAndParent.Property.GetNameCPP().Equals(TEXT("LayerBottomAltitude"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("LayerHeight"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("TracingStartMaxDistance"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("TracingMaxDistance"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("GroundAlbedo"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("Material"));
		}
		return true;
	}
	else if (OwnerStruct && OwnerStruct->GetName().Equals(TEXT("ExponentialHeightFogData")))
	{
		if (bShowMinimalOnly)
		{
			return false;
		}
		return true;
	}
	else if (OwnerClass == UExponentialHeightFogComponent::StaticClass())
	{
		if (bShowMinimalOnly)
		{
			return PropertyAndParent.Property.GetNameCPP().Equals(TEXT("FogDensity"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("FogHeightFalloff"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("FogInscatteringLuminance"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("DirectionalInscatteringExponent"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("DirectionalInscatteringLuminance"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("StartDistance"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("FogCutoffDistance"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("bEnableVolumetricFog"));
		}
		return true;
	}
	else if (OwnerClass == ULightComponent::StaticClass()
		|| OwnerClass == ULightComponentBase::StaticClass()
		|| OwnerClass == UDirectionalLightComponent::StaticClass())
	{
		if (bShowMinimalOnly)
		{
			return PropertyAndParent.Property.GetNameCPP().Equals(TEXT("Intensity"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("LightColor"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("IndirectLightingSaturation"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("ShadowAmount"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("SpecularScale"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("DiffuseScale"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("IndirectLightingIntensity"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("VolumetricScatteringIntensity"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("AtmosphereSunDiskColorScale"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("CloudShadowStrength"))
				|| PropertyAndParent.Property.GetNameCPP().Equals(TEXT("CloudScatteredLuminanceScale"));
		}

		UE_LOGF(LogEditorEnvironmentLightingViewer, Log, "%ls - %ls", *PropertyAndParent.Property.GetNameCPP(), *PropertyAndParent.Property.GetAuthoredName());
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

