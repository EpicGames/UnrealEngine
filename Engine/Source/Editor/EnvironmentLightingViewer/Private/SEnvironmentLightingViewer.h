// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;
struct FGeometry;
struct FPropertyAndParent;

#define ENVLIGHT_MAX_DETAILSVIEWS 5

class SEnvironmentLightingViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEnvironmentLightingViewer)
	{
	}
	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param	InArgs			A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs);

	/** Gets the widget contents of the app */
	virtual TSharedRef<SWidget> GetContent();

	virtual ~SEnvironmentLightingViewer();

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	TSharedPtr<class IDetailsView> DetailsViews[ENVLIGHT_MAX_DETAILSVIEWS];
	FLinearColor DefaultForegroundColor;

	TSharedPtr<class SCheckBox> CheckBoxAtmosphericLightsOnly;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> ComboBoxDetailFilter;
	TArray<TSharedPtr<FString>> ComboBoxDetailFilterOptions;
	int32 SelectedComboBoxDetailFilterOptions;

	TSharedPtr<class SButton> ButtonCreateSkyLight;
	TSharedPtr<class SButton> ButtonCreateAtmosphericLight0;
	TSharedPtr<class SButton> ButtonCreateSkyAtmosphere;
	TSharedPtr<class SButton> ButtonCreateVolumetricCloud;
	TSharedPtr<class SButton> ButtonCreateHeightFog;

	FReply OnButtonCreateSkyLight();
	FReply OnButtonCreateDirectionalLight(uint32 Index);
	FReply OnButtonCreateSkyAtmosphere();
	FReply OnButtonCreateVolumetricCloud();
	FReply OnButtonCreateHeightFog();

	TSharedRef<SWidget> ComboBoxDetailFilterWidget(TSharedPtr<FString> InItem);
	void ComboBoxDetailFilterWidgetSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetSelectedComboBoxDetailFilterTextLabel() const;

	bool GetIsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const;
};
